{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Geometry Cache from Alembic File(Experimental)",
	"Description" : "Support Geometry Cache from Alembic file without importing",
	"Category" : "Rendering",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules": [
		{
			"Name": "GeometryCacheAbcFile",
			"Type": "Editor",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64",
				"Mac",
				"Linux"
			]
		}
	],
	"Plugins": [
		{
			"Name": "AlembicImporter",
			"Enabled": true
		},
		{
			"Name": "GeometryCache",
			"Enabled": true
		}
	]
}
