// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryCacheAbcFile : ModuleRules
{
	public GeometryCacheAbcFile(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"AlembicLibrary",
				"Core",
				"CoreUObject",
				"Engine",
				"GeometryCache",
				"RenderCore",
				"RHI"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"EditorFramework",
				"GeometryCacheStreamer",
				"PropertyEditor",
				"Slate",
				"SlateCore",
                "UnrealEd"
            }
        );

		if (Target.bBuildWithEditorOnlyData)
		{
			PrivateDependencyModuleNames.Add("DerivedDataCache");
		}
	}
}
