// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorFactoryGeometryCacheAbcFile.h"
#include "GeometryCacheAbcFileActor.h"

UActorFactoryGeometryCacheAbcFile::UActorFactoryGeometryCacheAbcFile()
{
	DisplayName = FText::FromString( "Geometry Cache Abc File" );
	NewActorClass = AGeometryCacheAbcFileActor::StaticClass();
	bUseSurfaceOrientation = true;
}
