// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ActorFactories/ActorFactory.h"

#include "ActorFactoryGeometryCacheAbcFile.generated.h"

UCLASS()
class UActorFactoryGeometryCacheAbcFile : public UActorFactory
{
	GENERATED_BODY()

	UActorFactoryGeometryCacheAbcFile();
};
