// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheAbcFileActor.h"
#include "GeometryCacheAbcFileComponent.h"

AGeometryCacheAbcFileActor::AGeometryCacheAbcFileActor(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	GeometryCacheAbcFileComponent = CreateDefaultSubobject<UGeometryCacheAbcFileComponent>(TEXT("GeometryCacheAbcFileComponent"));
	RootComponent = GeometryCacheAbcFileComponent;
}
