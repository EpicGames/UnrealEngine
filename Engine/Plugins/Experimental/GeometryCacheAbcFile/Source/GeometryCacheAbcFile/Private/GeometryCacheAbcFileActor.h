// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"

#include "GeometryCacheAbcFileActor.generated.h"

class UGeometryCacheAbcFileComponent;

/** GeometryCacheAbcFile actor serves as a placeable actor for GeometryCache loading from an Alembic file */
UCLASS(ComponentWrapperClass)
class AGeometryCacheAbcFileActor : public AActor
{
	GENERATED_UCLASS_BODY()

private:
	UPROPERTY(Category = GeometryCacheAbcFileActor, VisibleAnywhere)
	TObjectPtr<UGeometryCacheAbcFileComponent> GeometryCacheAbcFileComponent;
};
