// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheAbcFileComponent.h"
#include "GeometryCache.h"
#include "GeometryCacheTrackAbcFile.h"
#include "GeometryCacheAbcFileSceneProxy.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "GeometryCacheAbcFileComponent"

UGeometryCacheAbcFileComponent::UGeometryCacheAbcFileComponent(const FObjectInitializer& ObjectInitializer)
{
	AbcSettings = ObjectInitializer.CreateDefaultSubobject<UAbcImportSettings>(this, TEXT("AbcSettings"));
	SamplingSettings.bSkipEmpty = true;
	NormalGenerationSettings.bForceOneSmoothingGroupPerObject = true;
}

#if WITH_EDITOR
void UGeometryCacheAbcFileComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	FProperty* PropertyThatChanged = PropertyChangedEvent.MemberProperty;
	const FName PropertyName = PropertyThatChanged ? PropertyThatChanged->GetFName() : NAME_None;

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UGeometryCacheAbcFileComponent, AlembicFilePath))
	{
		if (!AlembicFilePath.FilePath.IsEmpty())
		{
			InitializeGeometryCache();
		}
		else
		{
			if (GeometryCache && GeometryCache->Tracks.Num() != 0)
			{
				// Release the AbcFile resources
				UGeometryCacheTrackAbcFile* AbcFileTrack = Cast<UGeometryCacheTrackAbcFile>(GeometryCache->Tracks[0]);
				AbcFileTrack->SetSourceFile(FString(), nullptr);
			}

			GeometryCache = nullptr;
			MarkRenderStateDirty();
		}
		InvalidateTrackSampleIndices();
	}

	UMeshComponent::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

void UGeometryCacheAbcFileComponent::ReloadAbcFile()
{
	if (!GeometryCache || GeometryCache->Tracks.Num() == 0 || AlembicFilePath.FilePath.IsEmpty())
	{
		return;
	}

	UGeometryCacheTrackAbcFile* AbcFileTrack = Cast<UGeometryCacheTrackAbcFile>(GeometryCache->Tracks[0]);

	AbcSettings->ImportType = EAlembicImportType::GeometryCache;
	AbcSettings->SamplingSettings = SamplingSettings;
	AbcSettings->MaterialSettings = MaterialSettings;
	AbcSettings->ConversionSettings = ConversionSettings;
	AbcSettings->NormalGenerationSettings = NormalGenerationSettings;
	AbcSettings->GeometryCacheSettings = GeometryCacheSettings;

	FString FilePath(AlembicFilePath.FilePath);
	if (FPaths::IsRelative(FilePath))
	{
		FilePath = FPaths::Combine(FPaths::ProjectDir(), FilePath);
	}

	bool bIsValid = AbcFileTrack->SetSourceFile(FilePath, AbcSettings, GetAnimationTime(), bLooping);
	if (bIsValid)
	{
		// Update the SamplingSettings for the UI since the start/end frames may have changed
		SamplingSettings = AbcSettings->SamplingSettings;

		// Also store the number of frames in the cache
		GeometryCache->SetFrameStartEnd(SamplingSettings.FrameStart, SamplingSettings.FrameEnd);

		// Setup the materials from the AbcFile to the GeometryCache
		AbcFileTrack->SetupGeometryCacheMaterials(GeometryCache);
	}
	else
	{
		GeometryCache = nullptr;
		AlembicFilePath.FilePath.Empty();
	}

	ClearTrackData();
	SetupTrackData();

	MarkRenderStateDirty();
}

void UGeometryCacheAbcFileComponent::InitializeGeometryCache()
{
	if (!AlembicFilePath.FilePath.IsEmpty())
	{
		UGeometryCacheTrackAbcFile* AbcFileTrack = nullptr;
		if (!GeometryCache)
		{
			// Transient GeometryCache for use in current session
			GeometryCache = NewObject<UGeometryCache>();
			AbcFileTrack = NewObject<UGeometryCacheTrackAbcFile>(GeometryCache);
			GeometryCache->AddTrack(AbcFileTrack);
		}
		else
		{
			AbcFileTrack = Cast<UGeometryCacheTrackAbcFile>(GeometryCache->Tracks[0]);
		}

		if (AlembicFilePath.FilePath != AbcFileTrack->GetSourceFile())
		{
			ReloadAbcFile();
		}
	}
}

void UGeometryCacheAbcFileComponent::PostLoad()
{
	Super::PostLoad();

	InitializeGeometryCache();
}

FPrimitiveSceneProxy* UGeometryCacheAbcFileComponent::CreateSceneProxy()
{
	return new FGeometryCacheAbcFileSceneProxy(this);
}

#undef LOCTEXT_NAMESPACE
