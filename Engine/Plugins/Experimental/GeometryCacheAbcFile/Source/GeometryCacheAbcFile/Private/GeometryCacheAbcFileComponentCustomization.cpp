// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheAbcFileComponentCustomization.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "GeometryCacheAbcFileComponent.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "GeometryCacheAbcFileComponentCustomization"

TSharedRef<IDetailCustomization> FGeometryCacheAbcFileComponentCustomization::MakeInstance()
{
	return MakeShareable(new FGeometryCacheAbcFileComponentCustomization);
}

void FGeometryCacheAbcFileComponentCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> Objects;
	DetailBuilder.GetObjectsBeingCustomized(Objects);
	if (Objects.Num() != 1)
	{
		return;
	}

	GeometryCacheAbcFileComponent = Cast<UGeometryCacheAbcFileComponent>(Objects[0].Get());
	if (!GeometryCacheAbcFileComponent.Get())
	{
		return;
	}

	// Hide properties that are irrelevant to the GeometryCacheAbcFileComponent
	DetailBuilder.HideProperty("SamplingSettings.SamplingType");
	DetailBuilder.HideProperty("SamplingSettings.FrameSteps");
	DetailBuilder.HideProperty("SamplingSettings.TimeSteps");

	// Streaming Alembic doesn't do optimization nor compression
	DetailBuilder.HideProperty("GeometryCacheSettings.bApplyConstantTopologyOptimizations");
	DetailBuilder.HideProperty("GeometryCacheSettings.bOptimizeIndexBuffers");
	DetailBuilder.HideProperty("GeometryCacheSettings.CompressedPositionPrecision");
	DetailBuilder.HideProperty("GeometryCacheSettings.CompressedTextureCoordinatesNumberOfBits");

	// Add a button to reload the Alembic file and apply the settings
	IDetailCategoryBuilder& Category = DetailBuilder.EditCategory("Alembic");

	Category.AddCustomRow(LOCTEXT("AlembicSearchText", "Alembic"))
	.ValueContent()
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Left)
		[
			SNew(SButton)
			.Text(LOCTEXT("ReloadAbcFile", "Reload"))
			.ToolTipText(LOCTEXT("ReloadAbcFile_Tooltip", "Reload the Alembic file and apply the settings"))
			.OnClicked(this, &FGeometryCacheAbcFileComponentCustomization::ReloadAbcFile)
			.IsEnabled(this, &FGeometryCacheAbcFileComponentCustomization::IsReloadEnabled)
		]
	];
}

FReply FGeometryCacheAbcFileComponentCustomization::ReloadAbcFile()
{
	if (GeometryCacheAbcFileComponent.Get())
	{
		GeometryCacheAbcFileComponent->ReloadAbcFile();
	}

	return FReply::Handled();
}

bool FGeometryCacheAbcFileComponentCustomization::IsReloadEnabled() const
{
	if (GeometryCacheAbcFileComponent.Get())
	{
		return !GeometryCacheAbcFileComponent->AlembicFilePath.FilePath.IsEmpty();
	}
	return false;
}

#undef LOCTEXT_NAMESPACE
