// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"

class FReply;

class IDetailLayoutBuilder;

class FGeometryCacheAbcFileComponentCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	// IDetailCustomization interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
	// End of IDetailCustomization interface

protected:
	FReply ReloadAbcFile();
	bool IsReloadEnabled() const;

	TWeakObjectPtr<class UGeometryCacheAbcFileComponent> GeometryCacheAbcFileComponent;
};
