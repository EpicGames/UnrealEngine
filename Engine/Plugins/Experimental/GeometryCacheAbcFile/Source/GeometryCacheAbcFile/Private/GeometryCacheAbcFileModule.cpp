// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

#include "GeometryCacheAbcFileComponent.h"
#include "GeometryCacheAbcFileComponentCustomization.h"
#include "PropertyEditorModule.h"

class FGeometryCacheAbcFileModule : public IModuleInterface
{
	virtual void StartupModule() override
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(UGeometryCacheAbcFileComponent::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FGeometryCacheAbcFileComponentCustomization::MakeInstance));
	}

	virtual void ShutdownModule() override
	{
		if (UObjectInitialized() && !IsEngineExitRequested())
		{
			FPropertyEditorModule* PropertyModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
			if (PropertyModule)
			{
				PropertyModule->UnregisterCustomClassLayout(UGeometryCacheAbcFileComponent::StaticClass()->GetFName());
			}
		}
	}
};

IMPLEMENT_MODULE(FGeometryCacheAbcFileModule, GeometryCacheAbcFile);
