// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheAbcFileSceneProxy.h"
#include "AbcFile.h"
#include "GeometryCacheAbcFileComponent.h"
#include "GeometryCacheHelpers.h"
#include "GeometryCacheTrackAbcFile.h"
#include "SceneInterface.h"

FGeometryCacheAbcFileSceneProxy::FGeometryCacheAbcFileSceneProxy(UGeometryCacheAbcFileComponent* Component)
: FGeometryCacheSceneProxy(Component, [this]() { return new FGeomCacheTrackAbcFileProxy(GetScene().GetFeatureLevel()); })
{
}

bool FGeomCacheTrackAbcFileProxy::UpdateMeshData(float Time, bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	if (UGeometryCacheTrackAbcFile* AbcTrack = Cast<UGeometryCacheTrackAbcFile>(Track))
	{
		FGeometryCacheMeshData* PtrMeshData = nullptr;
		bool bResult = AbcTrack->UpdateMeshData(Time, bLooping, InOutMeshSampleIndex, PtrMeshData);
		if (bResult)
		{
			OutMeshData = *PtrMeshData;
		}
		return bResult;
	}
	return false;
}

bool FGeomCacheTrackAbcFileProxy::GetMeshData(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	if (UGeometryCacheTrackAbcFile* AbcTrack = Cast<UGeometryCacheTrackAbcFile>(Track))
	{
		return AbcTrack->GetMeshData(SampleIndex, OutMeshData);
	}
	return false;
}

bool FGeomCacheTrackAbcFileProxy::IsTopologyCompatible(int32 SampleIndexA, int32 SampleIndexB)
{
	if (UGeometryCacheTrackAbcFile* AbcTrack = Cast<UGeometryCacheTrackAbcFile>(Track))
	{
		return AbcTrack->IsTopologyCompatible(SampleIndexA, SampleIndexB);
	}
	return false;
}

const FVisibilitySample& FGeomCacheTrackAbcFileProxy::GetVisibilitySample(float Time, const bool bLooping) const
{
	// Assume the track is visible for its whole duration
	return FVisibilitySample::VisibleSample;
}

void FGeomCacheTrackAbcFileProxy::FindSampleIndexesFromTime(float Time, bool bLooping, bool bIsPlayingBackwards, int32 &OutFrameIndex, int32 &OutNextFrameIndex, float &InInterpolationFactor)
{
	if (UGeometryCacheTrackAbcFile* AbcTrack = Cast<UGeometryCacheTrackAbcFile>(Track))
	{
		int32 ThisFrameIndex = AbcTrack->FindSampleIndexFromTime(Time, bLooping);
		int32 LastFrameIndex = AbcTrack->GetEndFrameIndex();
		OutFrameIndex = ThisFrameIndex;
		OutNextFrameIndex = OutFrameIndex + 1;

		float AdjustedTime = Time;
		if (bLooping)
		{
			AdjustedTime = GeometyCacheHelpers::WrapAnimationTime(Time, AbcTrack->GetDuration());
		}

		// Time at ThisFrameIndex with index normalized to 0
		const float FrameIndexTime = (ThisFrameIndex - AbcTrack->GetAbcFile().GetStartFrameIndex()) * AbcTrack->GetAbcFile().GetSecondsPerFrame();
		InInterpolationFactor = (AdjustedTime - FrameIndexTime) / AbcTrack->GetAbcFile().GetSecondsPerFrame();

		// If playing backwards the logical order of previous and next is reversed
		if (bIsPlayingBackwards)
		{
			Swap(OutFrameIndex, OutNextFrameIndex);
			InInterpolationFactor = 1.0f - InInterpolationFactor;
		}
	}
}
