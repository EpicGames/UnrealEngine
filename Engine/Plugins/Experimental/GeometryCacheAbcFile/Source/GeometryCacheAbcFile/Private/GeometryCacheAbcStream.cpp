// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheAbcStream.h"
#include "AbcFile.h"
#include "AbcUtilities.h"
#include "DerivedDataCache.h"
#include "DerivedDataRequestOwner.h"
#include "GeometryCacheTrackAbcFile.h"
#include "HAL/IConsoleManager.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

static bool GAbcStreamCacheInDDC = true;
static FAutoConsoleVariableRef CVarAbcStreamCacheInDDC(
	TEXT("GeometryCache.Streamer.AbcStream.CacheInDDC"),
	GAbcStreamCacheInDDC,
	TEXT("Cache the streamed Alembic mesh data in the DDC"));

// Max read concurrency is 8 due to limitation in AbcFile
static int32 kAbcReadConcurrency = 8;

enum class EAbcStreamReadRequestStatus
{
	Scheduled,
	Completed,
	Cancelled
};

struct FGeometryCacheAbcStreamReadRequest
{
	FGeometryCacheMeshData* MeshData = nullptr;
	int32 ReadIndex = 0;
	int32 FrameIndex = 0;
	EAbcStreamReadRequestStatus Status = EAbcStreamReadRequestStatus::Scheduled;
};

// If AbcStream derived data needs to be rebuilt (new format, serialization
// differences, etc.) replace the version GUID below with a new one.
// In case of merge conflicts with DDC versions, you MUST generate a new GUID
// and set this new GUID as the version.
#define ABCSTREAM_DERIVED_DATA_VERSION TEXTVIEW("75887BC18F674774A53F91E4A57709D6")

class FAbcStreamDDCUtils
{
private:
	static FStringView GetAbcStreamDerivedDataVersion()
	{
		return ABCSTREAM_DERIVED_DATA_VERSION;
	}

	static UE::DerivedData::FCacheKey BuildDerivedDataKey(FStringView KeySuffix)
	{
		return UE::DerivedData::BuildLegacyCacheKey(TEXTVIEW("ABCSTREAM_"), GetAbcStreamDerivedDataVersion(), KeySuffix);
	}

	static FString BuildAbcStreamDerivedDataKeySuffix(UGeometryCacheTrackAbcFile* AbcTrack, int32 FrameIndex)
	{
		return AbcTrack->GetAbcTrackHash() + TEXT("_") + LexToString(FrameIndex);
	}

public:
	static UE::DerivedData::FCacheKey GetAbcStreamDDCKey(UGeometryCacheTrackAbcFile* AbcTrack, int32 FrameIndex)
	{
		return BuildDerivedDataKey(BuildAbcStreamDerivedDataKeySuffix(AbcTrack, FrameIndex));
	}
};

FGeometryCacheAbcStream::FGeometryCacheAbcStream(UGeometryCacheTrackAbcFile* InAbcTrack)
: FGeometryCacheStreamBase(
	kAbcReadConcurrency,
	FGeometryCacheStreamDetails{
		InAbcTrack->GetAbcFile().GetImportNumFrames(),
		InAbcTrack->GetAbcFile().GetImportLength(),
		InAbcTrack->GetAbcFile().GetSecondsPerFrame(),
		InAbcTrack->GetAbcFile().GetStartFrameIndex(),
		InAbcTrack->GetAbcFile().GetEndFrameIndex()})
, AbcTrack(InAbcTrack)
, Hash(InAbcTrack->GetAbcTrackHash())
{
}

void FGeometryCacheAbcStream::GetMeshData(int32 FrameIndex, int32 ConcurrencyIndex, FGeometryCacheMeshData& MeshData)
{
	// Get the mesh data straight from the Alembic file or from the DDC if it's already cached
	if (GAbcStreamCacheInDDC)
	{
		using namespace UE;
		using namespace UE::DerivedData;

		const FCacheKey DerivedDataKey = FAbcStreamDDCUtils::GetAbcStreamDDCKey(AbcTrack, FrameIndex);
		const FSharedString AbcFile(AbcTrack->GetSourceFile());

		FRequestOwner BlockingOwner(EPriority::Blocking);
		GetCache().GetValue({{AbcFile, DerivedDataKey}}, BlockingOwner, [this, FrameIndex, ConcurrencyIndex, &MeshData](FCacheGetValueResponse&& Response)
		{
			if (Response.Status == EStatus::Ok)
			{
				FSharedBuffer DerivedData = Response.Value.GetData().Decompress();
				FMemoryReaderView Ar(DerivedData, /*bIsPersistent=*/ true);
				Ar << MeshData;
			}
			else
			{
				FAbcUtilities::GetFrameMeshData(AbcTrack->GetAbcFile(), FrameIndex, MeshData, ConcurrencyIndex);

				FValue Value;
				{
					TArray<uint8> DerivedData;
					FMemoryWriter Ar(DerivedData, true);
					Ar << MeshData;
					Value = FValue::Compress(MakeSharedBufferFromArray(MoveTemp(DerivedData)));
				}

				FRequestOwner AsyncOwner(EPriority::Normal);
				GetCache().PutValue({{Response.Name, Response.Key, Value}}, AsyncOwner);
				AsyncOwner.KeepAlive();
			}
		});
		BlockingOwner.Wait();
	}
	else
	{
		// Synchronously load the requested frame data
		FAbcUtilities::GetFrameMeshData(AbcTrack->GetAbcFile(), FrameIndex, MeshData, ConcurrencyIndex);
	}
}
