// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GeometryCacheStreamBase.h"

struct FScriptContainerElement;

class UGeometryCacheTrackAbcFile;

class FGeometryCacheAbcStream : public FGeometryCacheStreamBase
{
public:
	FGeometryCacheAbcStream(UGeometryCacheTrackAbcFile* InAbcTrack);

protected:
	//~ Begin IGeometryCacheStream Interface
	virtual void GetMeshData(int32 FrameIndex, int32 ConcurrencyIndex, FGeometryCacheMeshData& OutMeshData) override;
	//~ End IGeometryCacheStream Interface

	UGeometryCacheTrackAbcFile* AbcTrack;
	FString Hash;
};
