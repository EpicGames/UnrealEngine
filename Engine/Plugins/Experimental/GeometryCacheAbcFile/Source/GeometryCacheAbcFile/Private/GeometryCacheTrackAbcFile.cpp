// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryCacheTrackAbcFile.h"

#include "AbcFile.h"
#include "AbcImporter.h"
#include "AbcImportLogger.h"
#include "AbcImportSettings.h"
#include "AbcUtilities.h"
#include "Framework/Notifications/NotificationManager.h"
#include "GeometryCacheAbcStream.h"
#include "GeometryCacheHelpers.h"
#include "GeometryCacheStreamerSettings.h"
#include "IGeometryCacheStreamer.h"
#include "Logging/TokenizedMessage.h"
#include "Misc/ArchiveMD5.h"
#include "Misc/Paths.h"
#include "PackageTools.h"
#include "UObject/Package.h"
#include "Widgets/Notifications/SNotificationList.h"
#include <Alembic/Abc/Base.h>
#include <Alembic/Abc/ErrorHandler.h>

DEFINE_LOG_CATEGORY_STATIC(LogGeometryCacheAbcFile, Log, All);

#define LOCTEXT_NAMESPACE "GeometryCacheTrackAbcFile"

namespace UE::GeometryCacheTrackAbcFile::Private
{
	void SerializeSettingsForDDC(FArchive& Ar, UAbcImportSettings* Settings)
	{
		// Include only settings that would affect how the frame data is generated
		Ar << Settings->ConversionSettings.bFlipU;
		Ar << Settings->ConversionSettings.bFlipV;
		Ar << Settings->ConversionSettings.Rotation;
		Ar << Settings->ConversionSettings.Scale;

		Ar << Settings->GeometryCacheSettings.bFlattenTracks;
		Ar << Settings->GeometryCacheSettings.bStoreImportedVertexNumbers;
		Ar << Settings->GeometryCacheSettings.MotionVectors;

		Ar << Settings->NormalGenerationSettings.bForceOneSmoothingGroupPerObject;
		Ar << Settings->NormalGenerationSettings.HardEdgeAngleThreshold;
		Ar << Settings->NormalGenerationSettings.bRecomputeNormals;
		Ar << Settings->NormalGenerationSettings.bIgnoreDegenerateTriangles;
		Ar << Settings->NormalGenerationSettings.bSkipComputingTangents;
	}

	FString ComputeSettingsHash(UAbcImportSettings* Settings)
	{
		if (!Settings)
		{
			return {};
		}

		FArchiveMD5 ArMD5;
		SerializeSettingsForDDC(ArMD5, Settings);

		FMD5Hash MD5Hash;
		ArMD5.GetHash(MD5Hash);

		return BytesToHex(MD5Hash.GetBytes(), MD5Hash.GetSize());
	}
}

UGeometryCacheTrackAbcFile::UGeometryCacheTrackAbcFile()
: StartFrameIndex(0)
, EndFrameIndex(0)
{
}

UGeometryCacheTrackAbcFile::~UGeometryCacheTrackAbcFile()
{
	IGeometryCacheStreamer::Get().UnregisterTrack(this);
}

const bool UGeometryCacheTrackAbcFile::UpdateMatrixData(const float Time, const bool bLooping, int32& InOutMatrixSampleIndex, FMatrix& OutWorldMatrix)
{
	if (AbcFile)
	{
		return Super::UpdateMatrixData(Time, bLooping, InOutMatrixSampleIndex, OutWorldMatrix);
	}
	return false;
}

const bool UGeometryCacheTrackAbcFile::UpdateMeshData(const float Time, const bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData*& OutMeshData)
{
	const int32 SampleIndex = FindSampleIndexFromTime(Time, bLooping);

	// If InOutMeshSampleIndex equals -1 (first creation) update the OutVertices and InOutMeshSampleIndex
	// Update the Vertices and Index if SampleIndex is different from the stored InOutMeshSampleIndex
	if (InOutMeshSampleIndex == -1 || SampleIndex != InOutMeshSampleIndex)
	{
		if (GetMeshData(SampleIndex, MeshData))
		{
			OutMeshData = &MeshData;
			InOutMeshSampleIndex = SampleIndex;
			return true;
		}
	}
	return false;
}

const bool UGeometryCacheTrackAbcFile::UpdateBoundsData(const float Time, const bool bLooping, const bool bIsPlayingBackward, int32& InOutBoundsSampleIndex, FBox& OutBounds)
{
	const int32 SampleIndex = FindSampleIndexFromTime(Time, bLooping);

	FGeometryCacheTrackSampleInfo SampledInfo = GetSampleInfo(Time, bLooping);
	if (InOutBoundsSampleIndex != SampleIndex)
	{
		OutBounds = SampledInfo.BoundingBox;
		InOutBoundsSampleIndex = SampleIndex;
		return true;
	}
	return false;
}

void UGeometryCacheTrackAbcFile::Reset()
{
	AbcFile.Reset();
	Hash.Empty();
	AbcStream.Reset();
	SampleInfos.Reset();

	EndFrameIndex = 0;
	Duration = 0.f;

	MatrixSamples.Reset();
	MatrixSampleTimes.Reset();

	MeshData = FGeometryCacheMeshData();
	MeshData.BoundingBox = FBox3f(ForceInit);
}

void UGeometryCacheTrackAbcFile::ShowNotification(const FText& Text)
{
	FNotificationInfo Info(Text);
	Info.bFireAndForget = true;
	Info.bUseLargeFont = false;
	Info.FadeOutDuration = 3.0f;
	Info.ExpireDuration = 7.0f;

	FSlateNotificationManager::Get().AddNotification(Info);
}

bool UGeometryCacheTrackAbcFile::SetSourceFile(const FString& FilePath, UAbcImportSettings* AbcSettings, float InitialTime, bool bIsLooping)
{
	IGeometryCacheStreamer& Streamer = IGeometryCacheStreamer::Get();
	Streamer.UnregisterTrack(this);
	Reset();

	if (!FilePath.IsEmpty())
	{
		AbcFile = MakeUnique<FAbcFile>(FilePath);
		EAbcImportError Result = AbcFile->Open();

		const FString Filename = FPaths::GetCleanFilename(FilePath);

		if (Result != EAbcImportError::AbcImportError_NoError)
		{
			Reset();

			FText FailureMessage = LOCTEXT("OpenFailureReason_Unknown", "Unknown open failure");
			switch (Result)
			{
			case EAbcImportError::AbcImportError_InvalidArchive:
				FailureMessage = LOCTEXT("OpenFailureReason_InvalidArchive", "Not a valid Alembic file");
				break;
			case EAbcImportError::AbcImportError_NoValidTopObject:
				FailureMessage = LOCTEXT("OpenFailureReason_InvalidRoot", "Alembic file has no valid root node");
				break;
			}
			UE_LOGF(LogGeometryCacheAbcFile, Warning, "Failed to open %ls: %ls", *Filename, *FailureMessage.ToString());

			return false;
		}

		// Automatically set the FrameEnd the same way as it's computed in FAbcImporter::GetEndFrameIndex when importing as GeometryCache to have same duration
		if (AbcSettings->SamplingSettings.FrameEnd == 0)
		{
			AbcSettings->SamplingSettings.FrameEnd = FMath::Max(AbcFile->GetMaxFrameIndex() - 1, 1);
		}

		Result = AbcFile->Import(AbcSettings);

		// The hash is composed of the Alembic file hash and the settings hash used to import it
		FString AbcHash;
		for (const FAbcFile::FMetaData& MetaData : AbcFile->GetArchiveMetaData())
		{
			if (MetaData.Key == TEXT("Abc.Hash"))
			{
				AbcHash = MetaData.Value;
				break;
			}
		}

		FString SettingsHash = UE::GeometryCacheTrackAbcFile::Private::ComputeSettingsHash(AbcSettings);
		Hash = AbcHash + TEXT("_") + SettingsHash;

		// Set the start/end frame after import since it might have been modified due to validation at import
		StartFrameIndex = AbcSettings->SamplingSettings.FrameStart;
		EndFrameIndex = AbcSettings->SamplingSettings.FrameEnd;

		if (Result != EAbcImportError::AbcImportError_NoError)
		{
			Reset();

			FText FailureMessage = LOCTEXT("LoadFailureReason_Unknown", "Unknown load failure");
			TArray<TSharedRef<FTokenizedMessage>> Messages = FAbcImportLogger::RetrieveMessages();
			if (Messages.Num() > 0)
			{
				FailureMessage = Messages[0]->ToText();
			}
			UE_LOGF(LogGeometryCacheAbcFile, Warning, "Failed to load %ls: %ls", *Filename, *FailureMessage.ToString());

			ShowNotification(FText::Format(LOCTEXT("LoadErrorNotification", "{0} could not be loaded. See Output Log for details."), FText::FromString(Filename)));

			return false;
		}

		TArray<FMatrix> Mats;
		Mats.Add(FMatrix::Identity);
		Mats.Add(FMatrix::Identity);

		TArray<float> MatTimes;
		MatTimes.Add(0.0f);
		MatTimes.Add(AbcFile->GetImportLength() + AbcFile->GetImportTimeOffset());
		SetMatrixSamples(Mats, MatTimes);

		Duration = AbcFile->GetImportLength();

		// Register this Track and associated Stream with the GeometryCacheStreamer and prefetch the first frame
		// The Stream ownership is passed to the Streamer
		AbcStream.Reset(new FGeometryCacheAbcStream(this));
		Streamer.RegisterTrack(this, AbcStream.Get());

		const int32 InitialFrameIndex = FindSampleIndexFromTime(InitialTime, bIsLooping);
		const float LookAhead = GetDefault<UGeometryCacheStreamerSettings>()->LookAheadBuffer;
		const int32 NumFrames = FMath::CeilToInt(LookAhead / AbcFile->GetSecondsPerFrame());

		AbcStream->Prefetch(InitialFrameIndex, NumFrames);
		GetMeshData(InitialFrameIndex, MeshData);

		if (MeshData.Positions.Num() == 0)
		{
			// This could happen if the Alembic has geometry but they are set as invisible in the source
			ShowNotification(FText::Format(LOCTEXT("NoVisibleGeometry", "Warning: {0} has no visible geometry."), FText::FromString(Filename)));
		}
	}

	SourceFile = FilePath;
	return true;
}

const int32 UGeometryCacheTrackAbcFile::FindSampleIndexFromTime(const float Time, const bool bLooping) const
{
	if (AbcFile)
	{
		float SampleTime = Time;
		if (bLooping)
		{
			SampleTime = GeometyCacheHelpers::WrapAnimationTime(Time, Duration);
		}
		return AbcFile->GetFrameIndex(SampleTime);
	}
	return 0;
}

const FGeometryCacheTrackSampleInfo& UGeometryCacheTrackAbcFile::GetSampleInfo(float Time, bool bLooping)
{
	if (SampleInfos.Num() == 0)
	{
		if (EndFrameIndex > StartFrameIndex)
		{
			// +1 because the SampleInfo for EndFrameIndex is also needed
			SampleInfos.SetNum(EndFrameIndex - StartFrameIndex + 1);
		}
		else
		{
			return FGeometryCacheTrackSampleInfo::EmptySampleInfo;
		}
	}

	// The sample info index must start from 0, while the sample index is between the range of the animation
	const int32 SampleIndex = FindSampleIndexFromTime(Time, bLooping);
	const int32 SampleInfoIndex = SampleIndex - StartFrameIndex;

	FGeometryCacheTrackSampleInfo& CurrentSampleInfo = SampleInfos[SampleInfoIndex];

	if (CurrentSampleInfo.SampleTime == 0.0f && CurrentSampleInfo.NumVertices == 0 && CurrentSampleInfo.NumIndices == 0)
	{
		if (GetMeshData(SampleIndex, MeshData))
		{
			CurrentSampleInfo = FGeometryCacheTrackSampleInfo(
				Time,
				(FBox) MeshData.BoundingBox,
				MeshData.Positions.Num(),
				MeshData.Indices.Num()
			);
		}
	}

	return CurrentSampleInfo;
}

const FGeometryCacheTrackSampleInfo& UGeometryCacheTrackAbcFile::GetSampleInfo(int32 FrameIndex)
{
	if (AbcFile)
	{
		// FrameIndex is normalized to 0
		return GetSampleInfo((FrameIndex - StartFrameIndex) * AbcFile->GetSecondsPerFrame(), false);
	}
	return FGeometryCacheTrackSampleInfo::EmptySampleInfo;
}

bool UGeometryCacheTrackAbcFile::IsTopologyCompatible(int32 FrameA, int32 FrameB)
{
	// FrameA/B could be -1, meaning invalid frame
	if (SampleInfos.Num() > 0 && FrameA >= 0 && FrameB >= 0)
	{
		return GetSampleInfo(FrameA).NumVertices == GetSampleInfo(FrameB).NumVertices;
	}
	return false;
}

bool UGeometryCacheTrackAbcFile::GetMeshDataAtTime(float Time, FGeometryCacheMeshData& OutMeshData)
{
	const bool bLooping = true;
	const int32 SampleIndex = FindSampleIndexFromTime(Time, bLooping);
	return GetMeshData(SampleIndex, OutMeshData);
}

bool UGeometryCacheTrackAbcFile::GetMeshDataAtSampleIndex(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	return GetMeshData(SampleIndex, OutMeshData);
}

bool UGeometryCacheTrackAbcFile::GetMeshData(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData)
{
	if (AbcFile)
	{
		if (IGeometryCacheStreamer::Get().IsTrackRegistered(this))
		{
			return IGeometryCacheStreamer::Get().TryGetFrameData(this, SampleIndex, OutMeshData);
		}
	}
	return false;
}

void UGeometryCacheTrackAbcFile::SetupGeometryCacheMaterials(UGeometryCache* GeometryCache)
{
	if (AbcFile)
	{
		// Create package where the materials will be saved into
		static const FString DestinationPath(TEXT("/Game/GeometryCacheAbcFile/Materials"));
		FString Name = FPaths::GetBaseFilename(SourceFile);
		FString PackageName = UPackageTools::SanitizePackageName(FPaths::Combine(*DestinationPath, *Name, *Name));

		UPackage* Package = CreatePackage(*PackageName);
		Package->FullyLoad();

		FAbcUtilities::SetupGeometryCacheMaterials(*AbcFile, GeometryCache, Package);
	}
}

FAbcFile& UGeometryCacheTrackAbcFile::GetAbcFile()
{
	return *AbcFile.Get();
}

void UGeometryCacheTrackAbcFile::UpdateTime(float Time, bool bLooping)
{
	if (AbcStream)
	{
		int32 FrameIndex = FindSampleIndexFromTime(Time, bLooping);
		AbcStream->UpdateCurrentFrameIndex(FrameIndex);
	}
}

#undef LOCTEXT_NAMESPACE
