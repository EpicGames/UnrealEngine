// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AbcImportSettings.h"
#include "GeometryCacheComponent.h"

#include "GeometryCacheAbcFileComponent.generated.h"

class UGeometryCache;

/** GeometryCacheAbcFileComponent, encapsulates a transient GeometryCache asset instance that fetches its data from an Alembic file and implements functionality for rendering and playback */
UCLASS(ClassGroup = (Rendering), meta = (BlueprintSpawnableComponent, DisplayName = "Geometry Cache Alembic File"), Experimental, ClassGroup = Experimental)
class GEOMETRYCACHEABCFILE_API UGeometryCacheAbcFileComponent : public UGeometryCacheComponent
{
	GENERATED_BODY()

	UGeometryCacheAbcFileComponent(const FObjectInitializer& ObjectInitializer);

public:

	UPROPERTY(EditAnywhere, Category = "Alembic", meta = (FilePathFilter = "Alembic files (*.abc)|*.abc", RelativeToGameDir))
	FFilePath AlembicFilePath;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Alembic")
	FAbcSamplingSettings SamplingSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Alembic")
	FAbcConversionSettings ConversionSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Alembic")
	FAbcGeometryCacheSettings GeometryCacheSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Alembic")
	FAbcNormalGenerationSettings NormalGenerationSettings;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Alembic")
	FAbcMaterialSettings MaterialSettings;

public:
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	//~ Begin UObject Interface
	virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin UPrimitiveComponent Interface.
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	//~ End UPrimitiveComponent Interface.

	void ReloadAbcFile();

protected:
	void InitializeGeometryCache();

	UPROPERTY(Transient)
	TObjectPtr<UAbcImportSettings> AbcSettings;
};
