// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCacheSceneProxy.h"

/** Scene proxy that uses FGeomCacheTrackAbcFileProxy */
class FGeometryCacheAbcFileSceneProxy final : public FGeometryCacheSceneProxy
{
public:
	FGeometryCacheAbcFileSceneProxy(class UGeometryCacheAbcFileComponent* Component);
};

/** Track proxy that gets its data from a GeometryCacheAbcFileTrack */
class FGeomCacheTrackAbcFileProxy : public FGeomCacheTrackProxy
{
public:
	FGeomCacheTrackAbcFileProxy(ERHIFeatureLevel::Type InFeatureLevel)
		: FGeomCacheTrackProxy(InFeatureLevel)
	{}

	//~ Begin FGeomCacheTrackProxy Interface
	virtual bool UpdateMeshData(float Time, bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData& OutMeshData) override;
	virtual bool GetMeshData(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData) override;
	virtual bool IsTopologyCompatible(int32 SampleIndexA, int32 SampleIndexB) override;
	virtual const FVisibilitySample& GetVisibilitySample(float Time, const bool bLooping) const override;
	virtual void FindSampleIndexesFromTime(float Time, bool bLooping, bool bIsPlayingBackwards, int32 &OutFrameIndex, int32 &OutNextFrameIndex, float &InterpolationFactor) override;
	//~ End FGeomCacheTrackProxy Interface
};
