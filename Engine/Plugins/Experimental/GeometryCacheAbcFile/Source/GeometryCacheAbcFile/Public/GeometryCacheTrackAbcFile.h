// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AbcFile.h"
#include "GeometryCacheAbcStream.h"
#include "GeometryCacheTrack.h"
#include "GeometryCacheMeshData.h"

#include "GeometryCacheTrackAbcFile.generated.h"

/** GeometryCacheTrack for Alembic file querying */
UCLASS(collapsecategories, hidecategories = Object, BlueprintType, config = Engine)
class GEOMETRYCACHEABCFILE_API UGeometryCacheTrackAbcFile : public UGeometryCacheTrack
{
	GENERATED_BODY()

	UGeometryCacheTrackAbcFile();
	virtual ~UGeometryCacheTrackAbcFile();

public:

	//~ Begin UGeometryCacheTrack Interface.
	virtual const bool UpdateMatrixData(const float Time, const bool bLooping, int32& InOutMatrixSampleIndex, FMatrix& OutWorldMatrix) override;
	virtual const bool UpdateMeshData(const float Time, const bool bLooping, int32& InOutMeshSampleIndex, FGeometryCacheMeshData*& OutMeshData) override;
	virtual const bool UpdateBoundsData(const float Time, const bool bLooping, const bool bIsPlayingBackward, int32& InOutBoundsSampleIndex, FBox& OutBounds) override;
	virtual const FGeometryCacheTrackSampleInfo& GetSampleInfo(float Time, const bool bLooping) override;
	virtual bool GetMeshDataAtTime(float Time, FGeometryCacheMeshData& OutMeshData) override;
	virtual bool GetMeshDataAtSampleIndex(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData) override;
	virtual void UpdateTime(float Time, bool bLooping) override;
	//~ End UGeometryCacheTrack Interface.

	bool SetSourceFile(const FString& FilePath, class UAbcImportSettings* AbcSettings, float InitialTime = 0.f, bool bIsLooping = true);
	const FString& GetSourceFile() const { return SourceFile; }

	const int32 FindSampleIndexFromTime(const float Time, const bool bLooping) const;

	int32 GetEndFrameIndex() const { return EndFrameIndex;  }

	bool GetMeshData(int32 SampleIndex, FGeometryCacheMeshData& OutMeshData);

	void SetupGeometryCacheMaterials(class UGeometryCache* GeometryCache);

	FAbcFile& GetAbcFile();

	const FString& GetAbcTrackHash() const { return Hash; }

	bool IsTopologyCompatible(int32 FrameA, int32 FrameB);

private:
	const FGeometryCacheTrackSampleInfo& GetSampleInfo(int32 FrameIndex);

	void Reset();
	void ShowNotification(const FText& Text);

private:
	FGeometryCacheMeshData MeshData;
	TArray<FGeometryCacheTrackSampleInfo> SampleInfos;
	TUniquePtr<FAbcFile> AbcFile;
	FString SourceFile;
	FString Hash;

	int32 StartFrameIndex;
	int32 EndFrameIndex;

	TUniquePtr<FGeometryCacheAbcStream> AbcStream;
};
